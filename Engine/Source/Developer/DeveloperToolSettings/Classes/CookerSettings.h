// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	CookerSettings.h: Declares the UCookerSettings class.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Engine/DeveloperSettings.h"
#include "CookerSettings.generated.h"

#define UE_API DEVELOPERTOOLSETTINGS_API

struct FPropertyChangedEvent;

UENUM()
enum class ECookProgressDisplayMode : int32
{
	/** Don't display any progress messages */
	Nothing = 0,

	/** Display the number of remaining packages */
	RemainingPackages = 1,

	/** Display names of cooked packages */
	PackageNames = 2,

	/** Display the number of remaining packages and package names */
	NamesAndRemainingPackages = 3,

	/** Display names and instigators of cooked packages */
	Instigators = 4,

	/** Display instigators and count of remaining packages */
	InstigatorsAndCount = 5,

	/** Display instigators and names of cooked packages */
	InstigatorsAndNames = 6,

	/** Display instigators and names and count of cooked packages */
	InstigatorsAndNamesAndCount = 7,

	/** MAX - invalid */
	Max UMETA(Hidden),
};

UENUM()
enum class EBlueprintComponentDataCookingMethod
{
	/** Do not generate optimized component data. No additional memory will be used. */
	Disabled,

	/** Generate optimized component data for all Blueprint types. This option will require the most additional memory. */
	AllBlueprints,

	/** Generate optimized component data only for Blueprint types that have explicitly enabled this feature in the class settings. */
	EnabledBlueprintsOnly,
};

UENUM()
enum class EBlueprintPropertyGuidsCookingMethod
{
	/** Do not include the property GUIDs in a cooked build. No additional memory will be used. */
	Disabled,

	/** Include the property GUIDs in a cooked build for all Blueprint types. This option will require the most additional memory. */
	AllBlueprints,

	/** Include the property GUIDs in a cooked build only for Blueprint types that have explicitly enabled this feature in their class settings. */
	EnabledBlueprintsOnly,
};

UENUM()
enum class ETextureFormatASTCCompressor
{
	/** ThirdParty/Intel/ISPCTexComp */
	IntelISPC,

	/** /ThirdParty/ARM */
	Arm,

	Max UMETA(Hidden),
};
/**
 * Various cooker settings.
 */
UCLASS(MinimalAPI, config=Engine, defaultconfig, meta=(DisplayName="Cooker"))
class UCookerSettings : public UDeveloperSettings
{
	GENERATED_UCLASS_BODY()

	UE_API virtual void PostInitProperties() override;
#if WITH_EDITOR
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif

	/**
	 * This class was moved from UnrealEd module, but to allow it to be used by developer tools, like UFE, it has moved to this module.
	 * However, for back-compat, we want to use the old name in the ini files, so that everything works without needing to touch every
	 * Game.ini file
	 */
	virtual void OverrideConfigSection(FString& InOutSectionName) override
	{
		InOutSectionName = TEXT("/Script/UnrealEd.CookerSettings");
	}

public:

	UPROPERTY(GlobalConfig, EditAnywhere, Category = Cooker, meta = (DisplayName = "Enable cooking via network in the background of the editor, launch on uses this setting, requires device to have network access to editor", ConfigRestartRequired = true))
	bool bEnableCookOnTheSide;

	UPROPERTY(GlobalConfig, EditAnywhere, Category = Cooker, meta = (DisplayName = "Generate DDC data in background for desired launch on platform (speeds up launch on)"))
	bool bEnableBuildDDCInBackground;

	/** Enable IncrementalCooks for launch on */
	UPROPERTY(GlobalConfig, EditAnywhere, Category = Cooker, meta = (DisplayName = "Use Incremental Cook for builds launched from the editor (launch on)"))
	bool bIterativeCookingForLaunchOn;

	/** Enable IncrementalCooks when triggering from Platforms dropdown menu */
	UPROPERTY(GlobalConfig, EditAnywhere, Category = Cooker, meta = (DisplayName = "Use Incremental Cook for the Platforms->Cook Content menu item"))
	bool bIterativeCookingForFileCookContent;

	/** Enable -cookonthefly for launch on */
	UPROPERTY(GlobalConfig, EditAnywhere, Category = Cooker, meta = (DisplayName = "Cooking on the fly when launching from the editor (launch on)"))
	bool bCookOnTheFlyForLaunchOn;

	UPROPERTY(GlobalConfig, EditAnywhere, Category = Cooker, meta = (
		ConsoleVariable = "cook.displaymode", DisplayName = "Cooker Progress Display Mode",
		ToolTip = "Controls log output of the cooker"))
	ECookProgressDisplayMode CookProgressDisplayMode;

	/** Ignore ini changes when doing iterative cooking, either in editor or out of editor */
	UPROPERTY(GlobalConfig, EditAnywhere, Category = Cooker, AdvancedDisplay)
	bool bIgnoreIniSettingsOutOfDateForIteration;

	/** Ignore native header file changes when doing iterative cooking, either in editor or out of editor */
	UPROPERTY(GlobalConfig, EditAnywhere, Category = Cooker, AdvancedDisplay)
	bool bIgnoreScriptPackagesOutOfDateForIteration;

	/** Whether or not to compile Blueprints in development mode when cooking. */
	UPROPERTY(GlobalConfig, EditAnywhere, Category = Cooker, AdvancedDisplay)
	bool bCompileBlueprintsInDevelopmentMode;

	/** Generate optimized component data to speed up Blueprint construction at runtime. This option can increase the overall Blueprint memory usage in a cooked build. Requires Event-Driven Loading (EDL), which is enabled by default. */
	UPROPERTY(GlobalConfig, EditAnywhere, Category = Cooker, AdvancedDisplay, meta = (DisplayName = "Generate optimized Blueprint component data"))
	EBlueprintComponentDataCookingMethod BlueprintComponentDataCookingMethod;

	/**
	 * Should we include the property GUIDs for a Blueprint class in a cooked build, so that SaveGame archives can redirect property names via the GUIDs?
	 * @note This option can increase the overall Blueprint memory usage in a cooked build, but can avoid needing to add CoreRedirect data for Blueprint classes stored within SaveGame archives.
	 */
	UPROPERTY(GlobalConfig, EditAnywhere, Category = Cooker, AdvancedDisplay, meta = (DisplayName = "Cook Blueprint property GUIDs?"))
	EBlueprintPropertyGuidsCookingMethod BlueprintPropertyGuidsCookingMethod = EBlueprintPropertyGuidsCookingMethod::EnabledBlueprintsOnly;

	/** List of class names to exclude when cooking for dedicated server */
	UPROPERTY(GlobalConfig, EditAnywhere, Category = Cooker, AdvancedDisplay, meta = (DisplayName = "Classes excluded when cooking for dedicated server"))
	TArray<FString> ClassesExcludedOnDedicatedServer;

	/** List of module names to exclude when cooking for dedicated server */
	UPROPERTY(GlobalConfig, EditAnywhere, Category = Cooker, AdvancedDisplay, meta = (DisplayName = "Modules excluded when cooking for dedicated server"))
	TArray<FString> ModulesExcludedOnDedicatedServer;

	/** List of class names to exclude when cooking for dedicated client */
	UPROPERTY(GlobalConfig, EditAnywhere, Category = Cooker, AdvancedDisplay, meta = (DisplayName = "Classes excluded when cooking for dedicated client"))
	TArray<FString> ClassesExcludedOnDedicatedClient;

	/** List of module names to exclude when cooking for dedicated client */
	UPROPERTY(GlobalConfig, EditAnywhere, Category = Cooker, AdvancedDisplay, meta = (DisplayName = "Modules excluded when cooking for dedicated client"))
	TArray<FString> ModulesExcludedOnDedicatedClient;

	/** List of r values that need to be versioned */
	UPROPERTY(GlobalConfig, EditAnywhere, Category = Cooker, AdvancedDisplay, meta = (DisplayName = "r values that need to be versioned"))
	TArray<FString> VersionedIntRValues;

	/** Quality of 0 means fastest, 3 means best quality */
	UPROPERTY(GlobalConfig, EditAnywhere, Category = Textures, meta = (DisplayName = "ASTC Compression Speed (0-3, 0 is fastest)"))
	int32 DefaultASTCQualityBySpeed;

	/** Quality of 0 means smallest (12x12 block size), 4 means best (4x4 block size) */
	UPROPERTY(GlobalConfig, EditAnywhere, Category = Textures, meta = (DisplayName = "ASTC Compression Block Size (0-4, 0 is smallest)"))
	int32 DefaultASTCQualityBySize;
	
	/** Quality of 0 means smallest (12x12 block size), 4 means best (4x4 block size) */
	UPROPERTY(GlobalConfig, EditAnywhere, Category = Textures, meta = (DisplayName = "ASTC Compression Block Size for High Quality (BC7) (0-4, 0 is smallest)"))
	int32 DefaultASTCQualityBySizeHQ;

	/** Whether to allow ASTC HDR profile */
	UPROPERTY(GlobalConfig, EditAnywhere, Category = Textures, meta = (
		ConsoleVariable = "cook.AllowASTCHDRProfile", DisplayName = "ASTC HDR Profile",
		ToolTip = "Whether to allow ASTC HDR profile, the HDR format is only supported on some devices, e.g. Apple A13, Mali-G72, Adreno (TM) 660",
		ConfigRestartRequired = true))
	uint32 bAllowASTCHDRProfile : 1;

	/** Allows opening cooked assets in the editor */
	UPROPERTY(EditAnywhere, config, Category = Editor, meta = (
		ConsoleVariable = "cook.AllowCookedDataInEditorBuilds", DisplayName = "Allow Cooked Content In The Editor",
		ToolTip = "If true, the editor will be able to open cooked assets (limited to a subset of supported asset types).",
		ConfigRestartRequired = true))
	uint32 bAllowCookedDataInEditorBuilds : 1;

private:
	/** Deprecated. Use BlueprintComponentDataCookingMethod instead. */
	UPROPERTY(GlobalConfig)
	bool bCookBlueprintComponentTemplateData;
};

#undef UE_API
