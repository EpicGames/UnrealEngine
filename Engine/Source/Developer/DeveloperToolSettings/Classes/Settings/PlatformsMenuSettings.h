// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "ProjectPackagingSettings.h"
#include "PlatformsMenuSettings.generated.h"

#define UE_API DEVELOPERTOOLSETTINGS_API

// ProjectPackagingSettings has two arrays of custom builds: EngineCustomBuilds and ProjectCustomBuilds. That makes it
// possible to have two builds with the same name (one in each list). For this reason, we need to store the settings
// separately for both.
enum class ECustomBuildType : uint8
{
	EngineCustomBuilds,
	ProjectCustomBuilds,
};

UCLASS(MinimalAPI, config=Game)
class UPlatformsMenuSettings
	: public UObject
{
	GENERATED_UCLASS_BODY()
	
	/** The directory to which the packaged project will be copied. */
	UPROPERTY(config, EditAnywhere, Category=Project)
	FDirectoryPath StagingDirectory;

	/** Name of the target to use for LaunchOn (only Game/Client targets) */
	UPROPERTY(config)
	FString LaunchOnTarget;

	/** Name of the platform (IniPlatformName) to use when the "Cook Content" button is clicked in the menu */
	UPROPERTY(config)
	FName CookPlatform;

	/** Build target to use when the "Cook Content" button is clicked in the menu */
	UPROPERTY(config)
	FString CookBuildTarget;

	/** Name of the platform (IniPlatformName) to use when the "Package Project" button is clicked in the menu */
	UPROPERTY(config)
	FName PackagePlatform;

	/** Build target to use when the "Package Project" button is clicked in the menu */
	UPROPERTY(config)
	FString PackageBuildTarget;

	/** Name of the platform (IniPlatformName) to use when the "Prepare for Debugging" button is clicked in the menu */
	UPROPERTY(config)
	FName PrepareForDebuggingPlatform;

	/** Build target to use when the "Prepare for Debugging" button is clicked in the menu */
	UPROPERTY(config)
	FString PrepareForDebuggingBuildTarget;

	/** Gets the current launch on target, checking that it's valid, and the default build target if it is not */
	UE_API const FTargetInfo* GetLaunchOnTargetInfo() const;
	
	UE_API FName GetTargetFlavorForPlatform(FName IniPlatformName) const;
	UE_API void SetTargetFlavorForPlatform(FName IniPlatformName, FName TargetFlavorName);

	UE_API FString GetArchitectureForPlatform(FName IniPlatformName) const;
	UE_API void SetArchitectureForPlatform(FName IniPlatformName, FString ArchitectureName);

	UE_API const FTargetInfo* GetBuildTargetInfo(const FString& TargetName, bool& bOutIsProjectTarget) const;

	UE_API EProjectPackagingBuildConfigurations GetLaunchOnBuildConfiguration() const;
	UE_API void SetLaunchOnBuildConfiguration(EProjectPackagingBuildConfigurations BuildConfiguration);

	UE_API EProjectPackagingBuildConfigurations GetPackageBuildConfiguration() const;
	UE_API void SetPackageBuildConfiguration(EProjectPackagingBuildConfigurations BuildConfiguration);

	/* Returns the IniPlatformName selected for the custom build */
	UE_API FName GetPlatformForCustomBuild(ECustomBuildType Type, const FString& CustomBuildName) const;
	UE_API void SetPlatformForCustomBuild(ECustomBuildType Type, const FString& CustomBuildName, FName IniPlatformName);

	UE_API FString GetDeviceIdForCustomBuild(ECustomBuildType Type, const FString& CustomBuildName) const;
	UE_API void SetDeviceIdForCustomBuild(ECustomBuildType Type, const FString& CustomBuildName, const FString& DeviceId);

	UE_API EProjectPackagingBuildConfigurations GetBuildConfigurationForCustomBuild(ECustomBuildType Type, const FString& CustomBuildName) const;
	UE_API void SetBuildConfigurationForCustomBuild(ECustomBuildType Type, const FString& CustomBuildName, EProjectPackagingBuildConfigurations BuildConfiguration);

	UE_API FString GetBuildTargetForCustomBuild(ECustomBuildType Type, const FString& CustomBuildName) const;
	UE_API void SetBuildTargetForCustomBuild(ECustomBuildType Type, const FString& CustomBuildName, const FString& TargetName);


private:
	/** Per platform flavor cooking target */
	UPROPERTY(config)
	TMap<FName, FName> PerPlatformTargetFlavorName;

	/** Per platform architecture */
	UPROPERTY(config, EditAnywhere, Category=Project)
	TMap<FName, FString> PerPlatformArchitecture;

	/** Platform name setting stored per entry in EngineCustomBuilds */
	UPROPERTY(config)
	TMap<FString, FName> PerEngineCustomBuildPlatformName;

	/** Platform name setting stored per entry in ProjectCustomBuilds */
	UPROPERTY(config)
	TMap<FString, FName> PerProjectCustomBuildPlatformName;

	/** Device ID setting stored per entry in EngineCustomBuilds */
	UPROPERTY(config)
	TMap<FString, FString> PerEngineCustomBuildDeviceId;

	/** Device ID setting stored per entry in ProjectCustomBuilds */
	UPROPERTY(config)
	TMap<FString, FString> PerProjectCustomBuildDeviceId;

	/** Build configuration setting stored per entry in EngineCustomBuilds */
	UPROPERTY(config)
	TMap<FString, EProjectPackagingBuildConfigurations> PerEngineCustomBuildBuildConfiguration;

	/** Build configuration setting stored per entry in ProjectCustomBuilds */
	UPROPERTY(config)
	TMap<FString, EProjectPackagingBuildConfigurations> PerProjectCustomBuildBuildConfiguration;

	/** Build target name setting stored per entry in EngineCustomBuilds */
	UPROPERTY(config)
	TMap<FString, FString> PerEngineCustomBuildBuildTargetName;

	/** Build target name setting stored per entry in ProjectCustomBuilds */
	UPROPERTY(config)
	TMap<FString, FString> PerProjectCustomBuildBuildTargetName;

	/** Build configuration to use for LaunchOn */
	UPROPERTY(config)
	TOptional<EProjectPackagingBuildConfigurations> LaunchOnBuildConfiguration;

	/** Build configuration to use when the "Package Project" button is clicked in the menu */
	UPROPERTY(config)
	TOptional<EProjectPackagingBuildConfigurations> PackageBuildConfiguration;
};

#undef UE_API
