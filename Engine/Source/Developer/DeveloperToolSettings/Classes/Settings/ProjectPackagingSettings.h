// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "ProjectPackagingSettings.generated.h"

#define UE_API DEVELOPERTOOLSETTINGS_API

struct FTargetInfo;

/**
 * Enumerates the available build configurations for project packaging.
 */
UENUM()
enum class EProjectPackagingBuildConfigurations : uint8
{
	/** Debug configuration. */
	PPBC_Debug UMETA(DisplayName="Debug"),

	/** DebugGame configuration. */
	PPBC_DebugGame UMETA(DisplayName="DebugGame"),

	/** Development configuration. */
	PPBC_Development UMETA(DisplayName="Development"),

	/** Test configuration. */
	PPBC_Test UMETA(DisplayName="Test"),

	/** Shipping configuration. */
	PPBC_Shipping UMETA(DisplayName="Shipping"),

	/** Number of entries in the enum. */
	PPBC_MAX UMETA(Hidden)
};

/**
 * Enumerates the available internationalization data presets for project packaging.
 */
UENUM()
enum class EProjectPackagingInternationalizationPresets : uint8
{
	/** English only. */
	English,

	/** English, French, Italian, German, Spanish. */
	EFIGS,

	/** English, French, Italian, German, Spanish, Chinese, Japanese, Korean. */
	EFIGSCJK,

	/** Chinese, Japanese, Korean. */
	CJK,

	/** All known cultures. */
	All
};

/**
 * Determines whether to build the executable when packaging. Note the equivalence between these settings and EPlayOnBuildMode.
 */
UENUM()
enum class EProjectPackagingBuild
{
	/** Always build. */
	Always UMETA(DisplayName="Always"),

	/** Never build. */
	Never UMETA(DisplayName="Never"),

	/** Default (if the never build.) */
	IfProjectHasCode UMETA(DisplayName="If project has code, or running a locally built editor"),

	/** If we're not packaging from a promoted or installed build. */
	IfEditorWasBuiltLocally UMETA(DisplayName="If running a locally built editor")
};

/**
* Enumerates the available methods for Blueprint nativization during project packaging.
*/
UENUM()
enum class EProjectPackagingBlueprintNativizationMethod : uint8
{
	/** Disable Blueprint nativization (default). */
	Disabled,

	/** Enable nativization for all Blueprint assets. */
	Inclusive,

	/** Enable nativization for selected Blueprint assets only. */
	Exclusive
};

/**
* The list of possible registry writebacks. During staging, iostore can
* optionally write back data that is only available during the staging process
* so that asset registry tools can associate this data with their respective
* assets.
* 
* Note that this is used in UnrealPak and thus can't use StaticEnum<>, so if you
* add any types here, be sure to add the parsing of the strings to IoStoreUtilities.cpp.
*/
UENUM()
enum class EAssetRegistryWritebackMethod : uint8
{
	/** Do not write-back staging metadata to the asset registry */
	Disabled,

	/** The development asset registry from the source cooked directory will be re-used. */
	OriginalFile,

	/** A duplicate asset registry will be created with the metadata added to it, adjacent to the cooked development asset registry. */
	AdjacentFile	
};

USTRUCT()
struct FPakOrderFileSpec
{
	GENERATED_BODY()

	UPROPERTY()
	FString Pattern;

	UPROPERTY()
	int32 Priority;

	FPakOrderFileSpec()
		: Priority(0)
	{
	}

	FPakOrderFileSpec(FString InPattern)
		: Pattern(MoveTemp(InPattern))
		, Priority(0)
	{
	}
};

USTRUCT()
struct FProjectBuildSettings
{
	GENERATED_BODY();

	/** The name for this custom build. It will be shown in menus for selection. */
	UPROPERTY(EditAnywhere, Category="Packaging")
	FString Name;

	/** Any help that you would like to include in the ToolTip of the menu option (or shown in interactive mode Turnkey) */
	UPROPERTY(EditAnywhere, Category="Packaging")
	FString HelpText;

	/** If this build step is specific to one or more platforms, add them here by name (note: use Windows, not Win64) */
	UPROPERTY(EditAnywhere, Category="Packaging")
	TArray<FString> SpecificPlatforms;

	/**
	 * The commandline for BuildCookRun UAT command. Some substitutions are allowed:
	 *   {Project} - Replaced with the path to the project's .uproject file
	 *   {Platform} - Replaced with the platform name this is run for
	 *   {inivalue:Config:Section:Key} - Replaced with the value for Key in Config's Section. Ex: -archivedirectory={inivalue:Engine:CustomSettings:OverrideArchiveDir}
	 *   {iniif:Token:Config:Section:Key} - Replaced with Token if the vlaue for Key in Config's Section evaluates to True. Ex: {iniif:-iostore:/Script/UnrealEd.ProjectPackagingSettings:bUseIoStore}
	 * Because ProjectPackagingSettings is a common section to read, if Config:Section: are not specified for 'iniif' or 'inivalue', it will use the ProjectPackagingSettings settings:
	 *   {iniif:-iostore:bUseIoStore}
	 * Additionally, the ini settings can have an optional search and replace modifier, to easily modify the string. The Replace can be blank:
	 *   {inivalue:BuildConfiguration|PPBC_=} - This will get the BuildConfiguration from the settings, and then remove the PPBC_ enum prefix from the string, to just get say Development
	 * 
	 * Adding -device={DeviceId} will show the command per-device in the platforms menu
	 */
	UPROPERTY(EditAnywhere, Category="Packaging")
	FString BuildCookRunParams;
};

/**
 * Implements the Editor's user settings.
 */
UCLASS(MinimalAPI, config=Game, defaultconfig)
class UProjectPackagingSettings
	: public UObject
{
	GENERATED_UCLASS_BODY()

	
	/**
	 * This class was moved from UnrealEd module, but to allow it to be used by developer tools, like UFE, it has moved to this module.
	 * However, for back-compat, we want to use the old name in the ini files, so that everything works without needing to touch every
	 * Game.ini file
	 */
	virtual void OverrideConfigSection(FString& InOutSectionName) override
	{
		InOutSectionName = TEXT("/Script/UnrealEd.ProjectPackagingSettings");
	}

public:

	/**
	 * Information about each packaging configuration
	 */
	struct FConfigurationInfo
	{
		EBuildConfiguration Configuration;
		FText Name;
		FText ToolTip;
	};

	/**
	 * Static array of information about each configuration
	 */
	static UE_API const FConfigurationInfo ConfigurationInfo[(int)EProjectPackagingBuildConfigurations::PPBC_MAX];

	/** Specifies whether to build the game executable during packaging. */
	UPROPERTY(config, EditAnywhere, Category=Project)
	EProjectPackagingBuild Build;

	/** The build configuration for which the project is packaged. */
	UPROPERTY(config, EditAnywhere, Category=Project)
	EProjectPackagingBuildConfigurations BuildConfiguration;

	/** Name of the target to build */
	UPROPERTY(config, EditAnywhere, Category=Project)
	FString BuildTarget;

	/**
	 * If enabled, a full rebuild will be enforced each time the project is being packaged.
	 * If disabled, only modified files will be built, which can improve iteration time.
	 * Unless you iterate on packaging, we recommend full rebuilds when packaging.
	 */
	UPROPERTY(config, EditAnywhere, Category=Project)
	bool FullRebuild;

	/**
	 * If enabled, a distribution build will be created and the shipping configuration will be used
	 * If disabled, a development build will be created
	 * Distribution builds are for publishing to the App Store
	 */
	UPROPERTY(config, EditAnywhere, Category=Project)
	bool ForDistribution;

	/** If enabled, debug files will be included in staged shipping builds. */
	UPROPERTY(config, EditAnywhere, Category=Project, meta = (DisplayName = "Include Debug Files in Shipping Builds"))
	bool IncludeDebugFiles;

	/** If enabled, all content will be put into a one or more .pak files instead of many individual files (default = enabled). */
	UPROPERTY(config, EditAnywhere, Category=Packaging)
	bool UsePakFile;

	/** If enabled, use .utoc/.ucas container files for staged/packaged package data instead of pak. */
	UPROPERTY(config, EditAnywhere, Category = Packaging)
	bool bUseIoStore;

	/** If enabled, use Zen Server for storing and fetching cooked data instead of using the local file system.  */
	UPROPERTY(config, EditAnywhere, Category = Packaging, meta=(DisplayName="Use Zen Server as cooked output store"))
	bool bUseZenStore;

	/** If enabled, staging will make a binary config file for faster loading. */
	UPROPERTY(config, EditAnywhere, Category = Packaging)
	bool bMakeBinaryConfig;

	/**
	 * If enabled, will generate pak file chunks.  Assets can be assigned to chunks in the editor or via a delegate (See ShooterGameDelegates.cpp). 
	 * Can be used for streaming installs (PS4 Playgo, XboxOne Streaming Install, etc)
	 */
	UPROPERTY(config, EditAnywhere, Category=Packaging)
	bool bGenerateChunks;

	/** 
	 * If enabled, no platform will generate chunks, regardless of settings in platform-specific ini files.
	 */
	UPROPERTY(config, EditAnywhere, Category=Packaging)
	bool bGenerateNoChunks;

	/**
	 * Normally during chunk generation all dependencies of a package in a chunk will be pulled into that package's chunk.
	 * If this is enabled then only hard dependencies are pulled in. Soft dependencies stay in their original chunk.
	 */
	UPROPERTY(config, EditAnywhere, Category = Packaging)
	bool bChunkHardReferencesOnly;

	/**
	 * If true, individual files are only allowed to be in a single chunk and it will assign it to the lowest number requested
	 * If false, it may end up in multiple chunks if requested by the cooker
	 */
	UPROPERTY(config, EditAnywhere, Category = Packaging, AdvancedDisplay)
	bool bForceOneChunkPerFile;

	/**
	 * If > 0 this sets a maximum size per chunk. Chunks larger than this size will be split into multiple pak files such as pakchunk0_s1
	 * This can be set in platform specific game.ini files
	 */
	UPROPERTY(config, EditAnywhere, Category = Packaging, AdvancedDisplay)
	int64 MaxChunkSize;

	/** 
	 * If enabled, will generate data for HTTP Chunk Installer. This data can be hosted on webserver to be installed at runtime. Requires "Generate Chunks" enabled.
	 */
	UPROPERTY(config, EditAnywhere, Category=Packaging)
	bool bBuildHttpChunkInstallData;

	/** 
	 * When "Build HTTP Chunk Install Data" is enabled this is the directory where the data will be build to.
	 */	
	UPROPERTY(config, EditAnywhere, Category = Packaging)
	FDirectoryPath HttpChunkInstallDataDirectory;

	/**
	* Whether to write staging metadata back to the asset registry. This metadata contains information such as
	* the actual compressed chunk sizes of the assets as well as some bulk data diff blame support information.
	*/
	UPROPERTY(config, EditAnywhere, Category = Packaging, AdvancedDisplay)
	EAssetRegistryWritebackMethod WriteBackMetadataToAssetRegistry;

	/**
	* Whether or not to write a json summary file that contains size information to the cooked Metadata directory
	*/
	UPROPERTY(config, EditAnywhere, Category = Packaging, AdvancedDisplay, meta = (EditCondition = "WriteBackMetadataToAssetRegistry != EAssetRegistryWritebackMethod::Disabled"))
	bool bWritePluginSizeSummaryJsons;

	/**
	 * Create compressed cooked packages (decreased deployment size)
	 */
	UPROPERTY(config, EditAnywhere, Category=Packaging, AdvancedDisplay, meta=(DisplayName="Create compressed cooked packages"))
	bool bCompressed;
	
	/**
	 * A comma separated list of formats to use for .pak file and IoStore compression. If more than one is specified, the list is in order of priority, with fallbacks to other formats
	 * in case of errors or unavailability of the format (plugin not enabled, etc).
	 * Commonly PackageCompressionFormat=Oodle or PackageCompressionFormat=None
	 */
	UPROPERTY(config, EditAnywhere, Category = Packaging, AdvancedDisplay, meta = (DisplayName = "Package Compression Format"))
	FString PackageCompressionFormat;
	
	/**
	 * Force use of PackageCompressionFormat (do not use override HardwareCompressionFormat from DDPI)
	 */
	UPROPERTY(config, EditAnywhere, Category=Packaging, AdvancedDisplay, meta=(DisplayName="Use this Compression Format not hardware override"))
	bool bForceUseProjectCompressionFormatIgnoreHardwareOverride;

	/**
	 * A generic setting for allowing a project to control compression settings during .pak file and iostore compression.
	 * For instance PackageAdditionalCompressionOptions=-compressionblocksize=1MB -asynccompression
	 */
	UPROPERTY(config, EditAnywhere, Category = Packaging, AdvancedDisplay, meta = (DisplayName = "Package Compression Commandline Options"))
	FString PackageAdditionalCompressionOptions;
	
	/**
	 * For compressors with multiple methods, select one.  eg. for Oodle you may use one of {Kraken,Mermaid,Selkie,Leviathan}
	 */
	UPROPERTY(config, EditAnywhere, Category = Packaging, AdvancedDisplay, meta = (DisplayName = "Package Compression Method"))
	FString PackageCompressionMethod;
		
	/*
	 * For compressors with variable levels, select the compressor effort level, which makes packages smaller but takes more time to encode.
	 * This does not affect decode speed.  For faster iteration, use lower effort levels (eg. 1)
	 */
	UPROPERTY(config, EditAnywhere, Category = Packaging, AdvancedDisplay, meta = (DisplayName = "Compressor Effort Level for Debug & Development"))
	int32 PackageCompressionLevel_DebugDevelopment;
	
	UPROPERTY(config, EditAnywhere, Category = Packaging, AdvancedDisplay, meta = (DisplayName = "Compressor Effort Level for Test & Shipping"))
	int32 PackageCompressionLevel_TestShipping;
	
	UPROPERTY(config, EditAnywhere, Category = Packaging, AdvancedDisplay, meta = (DisplayName = "Compressor Effort Level for Distribution"))
	int32 PackageCompressionLevel_Distribution;

	/**
	 * A generic setting which is used to determine whether it is worth using compression for a block of data when creating IoStore or .pak files.
	 * If the amount of saved bytes is smaller than the specified value, then the block of data remains uncompressed.
	 * The optimal value of this setting depends on the capabilities of the target platform. For instance PackageCompressionMinBytesSaved=1024
	 * Note that some compressors (for example Oodle) do their own internal worth it check and only use this value to determine the minimal size of a block which should be compressed.
	 */
	UPROPERTY(config, EditAnywhere, Category = Packaging, AdvancedDisplay, meta = (DisplayName = "Minimum amount of bytes which should be saved when compressing a block of data, otherwise data remains uncompressed"))
	int32 PackageCompressionMinBytesSaved;

	/**
	 * A generic setting which is used to determine whether it is worth using compression for a block of data when creating IoStore or .pak files.
	 * If the saved percentage of a compressed block of data is smaller than the specified value, then the block remains uncompressed.
	 * The optimal value of this setting depends on the capabilities of the target platform. For instance PackageCompressionMinPercentSaved=5
	 * Note that some compressors (for example Oodle) do their own internal worth it check and ignore this value.
	 */
	UPROPERTY(config, EditAnywhere, Category = Packaging, AdvancedDisplay, meta = (DisplayName = "Minimum percentage of a block of data which should be saved when performing compression, otherwise data remains uncompressed"))
	int32 PackageCompressionMinPercentSaved;

	/**
	 * Specifies if DDC should be used to store and retrieve compressed data when creating IoStore containers.
	 */
	UPROPERTY(config, EditAnywhere, Category = Packaging, AdvancedDisplay, meta = (DisplayName = "Enable DDC for IoStore compression"))
	bool bPackageCompressionEnableDDC;

	/**
	 * Specifies the minimum (uncompressed) size for storing a compressed IoStore chunk in DDC.
	 */
	UPROPERTY(config, EditAnywhere, Category = Packaging)
	int32 PackageCompressionMinSizeToConsiderDDC;

	/** 
	 * Version name for HTTP Chunk Install Data.
	 */
	UPROPERTY(config, EditAnywhere, Category = Packaging)
	FString HttpChunkInstallDataVersion;

	/** Specifies whether to include an installer for prerequisites of packaged games, such as redistributable operating system components, on platforms that support it. */
	UPROPERTY(config, EditAnywhere, Category=Prerequisites, meta=(DisplayName="Include prerequisites installer"))
	bool IncludePrerequisites;

	/** Specifies whether to include prerequisites alongside the game executable. */
	UPROPERTY(config, EditAnywhere, Category = Prerequisites, meta = (DisplayName = "Include app-local prerequisites"))
	bool IncludeAppLocalPrerequisites;

	/** 
	 * By default shader code gets saved inline inside material assets, 
	 * enabling this option will store only shader code once as individual files
	 * This will reduce overall package size but might increase loading time
	 */
	UPROPERTY(config, EditAnywhere, Category=Packaging)
	bool bShareMaterialShaderCode;

	/** 
	 * With this option off, the shader code will be stored in the library essentially in a random order,
	 * squarely the same in which the assets were loaded by the cooker. Enabling this will sort the shaders
	 * by their hash, which makes the shader library more similar between the builds which can help patching, but
	 * can adversely affect loading times.
	 */
	UPROPERTY(config, EditAnywhere, Category = Packaging, meta = (EditCondition = "bShareMaterialShaderCode"))
	bool bDeterministicShaderCodeOrder;

	/**
	 * By default shader shader code gets saved into individual platform agnostic files,
	 * enabling this option will use the platform-specific library format if and only if one is available
	 * This will reduce overall package size but might increase loading time
	 */
	UPROPERTY(config, EditAnywhere, Category=Packaging, meta = (EditCondition = "bShareMaterialShaderCode", ConfigRestartRequired = true))
	bool bSharedMaterialNativeLibraries;

	/** A directory containing additional prerequisite packages that should be staged in the executable directory. Can be relative to $(EngineDir) or $(ProjectDir) */
	UPROPERTY(config, EditAnywhere, Category=Prerequisites, AdvancedDisplay)
	FDirectoryPath ApplocalPrerequisitesDirectory;

	/**
	 * Specifies whether to include the crash reporter in the packaged project. 
	 * This is included by default for Blueprint based projects, but can optionally be disabled.
	 */
	UPROPERTY(config, EditAnywhere, Category=Packaging, AdvancedDisplay)
	bool IncludeCrashReporter;

	/** Predefined sets of culture whose internationalization data should be packaged. */
	UPROPERTY(config, EditAnywhere, Category=Packaging, AdvancedDisplay, meta=(DisplayName="Internationalization Support"))
	EProjectPackagingInternationalizationPresets InternationalizationPreset;

	/** Languages whose data should be cooked, staged, and packaged. */
	UPROPERTY(config, EditAnywhere, Category=Packaging, AdvancedDisplay, meta=(DisplayName="Languages to Package"))
	TArray<FString> CulturesToStage;

	/** List of localization targets that should be chunked during cooking (if using chunks) */
	UPROPERTY(config, EditAnywhere, Category=Packaging, AdvancedDisplay)
	TArray<FString> LocalizationTargetsToChunk;

	/** The chunk ID that should be used as the catch-all chunk for any non-asset localized strings */
	UPROPERTY(config, EditAnywhere, Category=Packaging, AdvancedDisplay)
	int32 LocalizationTargetCatchAllChunkId = 0;

	/**
	 * Cook all things in the project content directory
	 */
	UPROPERTY(config, EditAnywhere, Category=Packaging, AdvancedDisplay, meta=(DisplayName="Cook everything in the project content directory (ignore list of maps below)"))
	bool bCookAll;

	/**
	 * Cook only maps (this only affects the cookall flag)
	 */
	UPROPERTY(config, EditAnywhere, Category=Packaging, AdvancedDisplay, meta=(DisplayName="Cook only maps (this only affects cookall)"))
	bool bCookMapsOnly;

	/**
	 * Cook with Warnings As Errors
	 */
	UPROPERTY(config, EditAnywhere, Category=Packaging, AdvancedDisplay, meta=(DisplayName="Cook with Warnings As Errors enabled"))
	bool bTreatWarningsAsErrorsOnCook;

	/**
	* Encrypt ini files inside of the pak file
	* NOTE: Replaced by the settings inside the cryptokeys system. Kept here for legacy migration purposes.
	*/
	UPROPERTY(config)
	bool bEncryptIniFiles_DEPRECATED;

	/**
	* Encrypt the pak index
	* NOTE: Replaced by the settings inside the cryptokeys system. Kept here for legacy migration purposes.
	*/
	UPROPERTY(config)
	bool bEncryptPakIndex_DEPRECATED;

	/**
	 * Enable the early downloader pak file pakearly.txt
	 * This has been superseded by the functionality in DefaultPakFileRules.ini
	 */
	UPROPERTY(config)
	bool GenerateEarlyDownloaderPakFile_DEPRECATED;
	
	/**
	 * Don't include content in any editor folders when cooking.  This can cause issues with missing content in cooked games if the content is being used. 
	 */
	UPROPERTY(config, EditAnywhere, Category = Packaging, AdvancedDisplay, meta = (DisplayName = "Exclude editor content when cooking"))
	bool bSkipEditorContent;

	/**
	 * Don't include movies by default when staging/packaging
	 * Specific movies can be specified below, and this can be in a platform ini
	 */
	UPROPERTY(config, EditAnywhere, Category = Packaging, AdvancedDisplay, meta = (DisplayName = "Exclude movie files when staging"))
	bool bSkipMovies;

	/**
	 * If SkipMovies is true, these specific movies will still be added to the .pak file (if using a .pak file; otherwise they're copied as individual files)
	 * This should be the name with no extension
	 */
	UPROPERTY(config, EditAnywhere, Category = Packaging, AdvancedDisplay, meta = (DisplayName = "Specific movies to Package"))
	TArray<FString> UFSMovies;

	/**
	 * If SkipMovies is true, these specific movies will be copied when packaging your project, but are not supposed to be part of the .pak file
	 * This should be the name with no extension
	 */
	UPROPERTY(config, EditAnywhere, Category = Packaging, AdvancedDisplay, meta = (DisplayName = "Specific movies to Copy"))
	TArray<FString> NonUFSMovies;

	/**
	 * If set, only these specific pak files will be compressed. This should take the form of "*pakchunk0*"
	 * This can be set in a platform-specific ini file
	 */
	UPROPERTY(config, EditAnywhere, Category = Packaging, AdvancedDisplay)
	TArray<FString> CompressedChunkWildcard;

	UE_DEPRECATED(5.1, "This property is no longer supported. Use IniKeyDenylist.")
	UPROPERTY(config /*, EditAnywhere, Category = Packaging */, meta = (DeprecatedProperty, DeprecationMessage = "This property is no longer supported. Use IniKeyDenylist."))
	TArray<FString> IniKeyBlacklist;

	/** List of ini file keys to strip when packaging */
	UPROPERTY(config, EditAnywhere, Category = Packaging)
	TArray<FString> IniKeyDenylist;

	UE_DEPRECATED(5.1, "This property is no longer supported. Use IniSectionDenylist.")
	UPROPERTY(config /*, EditAnywhere, Category = Packaging */, meta = (DeprecatedProperty, DeprecationMessage = "This property is no longer supported. Use IniSectionDenylist."))
	TArray<FString> IniSectionBlacklist;

	/** List of ini file sections to strip when packaging */
	UPROPERTY(config, EditAnywhere, Category = Packaging)
	TArray<FString> IniSectionDenylist;

	/**
	 * List of specific files to include with GenerateEarlyDownloaderPakFile
	 */
	UPROPERTY(config)
	TArray<FString> EarlyDownloaderPakFileFiles_DEPRECATED;


	/**
	 * List of maps to include when no other map list is specified on commandline
	 */
	UPROPERTY(config, EditAnywhere, Category = Packaging, AdvancedDisplay, meta = (DisplayName = "List of maps to include in a packaged build", RelativeToGameContentDir, LongPackageName))
	TArray<FFilePath> MapsToCook;	

	/**
	 * Directories containing .uasset files that should always be cooked regardless of whether they're referenced by anything in your project
	 * These paths are stored either as a full package path (e.g. /Game/Folder, /Engine/Folder, /PluginName/Folder) or as a relative package path from /Game
	 */
	UPROPERTY(config, EditAnywhere, Category=Packaging, AdvancedDisplay, meta=(DisplayName="Additional Asset Directories to Cook", LongPackageName))
	TArray<FDirectoryPath> DirectoriesToAlwaysCook;

	/**
	 * Directories containing .uasset files that should never be cooked even if they are referenced by your project
	 * These paths are stored either as a full package path (e.g. /Game/Folder, /Engine/Folder, /PluginName/Folder) or as a relative package path from /Game
	 */
	UPROPERTY(config, EditAnywhere, Category = Packaging, AdvancedDisplay, meta = (DisplayName = "Directories to never cook", LongPackageName))
	TArray<FDirectoryPath> DirectoriesToNeverCook;

	/**
	 * Directories containing .uasset files that are for editor testing purposes and should not be included in
	 * enumerations of all packages in a root directory, because they will cause errors on load
	 * These paths are stored either as a full package path (e.g. /Game/Folder, /Engine/Folder, /PluginName/Folder) or as a relative package path from /Game
	 */
	UPROPERTY(config, EditAnywhere, Category = Packaging, AdvancedDisplay, meta = (DisplayName = "Test directories to not search", LongPackageName))
	TArray<FDirectoryPath> TestDirectoriesToNotSearch;

	/**
	 * Directories containing files that should always be added to the .pak file (if using a .pak file; otherwise they're copied as individual files)
	 * This is used to stage additional files that you manually load via the UFS (Unreal File System) file IO API
	 * Note: These paths are relative to your project Content directory
	 */
	UPROPERTY(config, EditAnywhere, Category=Packaging, AdvancedDisplay, meta=(DisplayName="Additional Non-Asset Directories to Package", RelativeToGameContentDir))
	TArray<FDirectoryPath> DirectoriesToAlwaysStageAsUFS;

	/**
	 * Directories containing files that should always be copied when packaging your project, but are not supposed to be part of the .pak file
	 * This is used to stage additional files that you manually load without using the UFS (Unreal File System) file IO API, eg, third-party libraries that perform their own internal file IO
	 * Note: These paths are relative to your project Content directory
	 */
	UPROPERTY(config, EditAnywhere, Category=Packaging, AdvancedDisplay, meta=(DisplayName="Additional Non-Asset Directories To Copy", RelativeToGameContentDir))
	TArray<FDirectoryPath> DirectoriesToAlwaysStageAsNonUFS;	

	/**
	 * Directories containing files that should always be added to the .pak file for a dedicated server (if using a .pak file; otherwise they're copied as individual files)
	 * This is used to stage additional files that you manually load via the UFS (Unreal File System) file IO API
	 * Note: These paths are relative to your project Content directory
	 */
	UPROPERTY(config, EditAnywhere, Category=Packaging, AdvancedDisplay, meta=(DisplayName="Additional Non-Asset Directories to Package for dedicated server only", RelativeToGameContentDir))
	TArray<FDirectoryPath> DirectoriesToAlwaysStageAsUFSServer;

	/**
	 * Directories containing files that should always be copied when packaging your project for a dedicated server, but are not supposed to be part of the .pak file
	 * This is used to stage additional files that you manually load without using the UFS (Unreal File System) file IO API, eg, third-party libraries that perform their own internal file IO
	 * Note: These paths are relative to your project Content directory
	 */
	UPROPERTY(config, EditAnywhere, Category=Packaging, AdvancedDisplay, meta=(DisplayName="Additional Non-Asset Directories To Copy for dedicated server only", RelativeToGameContentDir))
	TArray<FDirectoryPath> DirectoriesToAlwaysStageAsNonUFSServer;	

	/**
	 * A list of custom builds that will show up in the Platforms menu to allow customized builds that make sense for your project. Will show up near Package Project in the Platforms menu.
	 */
	UPROPERTY(config, EditAnywhere, Category=CustomBuilds, meta=(DisplayName = "Additional builds for this project.", TitleProperty="Name"))
	TArray<FProjectBuildSettings> ProjectCustomBuilds;

	/** If set, platforms that destructively edit the iostore containers during packaging will save a copy prior to doing so. */
	UPROPERTY(config, EditAnywhere, Category = Packaging)
	bool bRetainStagedDirectory;

	/**
	 * A list of custom builds, specified in engine ini files, and not editable in editor, that will show up in the Platforms menu to allow customized builds for all projects
	 */
	UPROPERTY(config)
	TArray<FProjectBuildSettings> EngineCustomBuilds;

	/**
	* The type name of a CustomStageCopyHandler subclass to instanciate during the copy build to staging directory step. See SetupCustomStageCopyHandler() in CopyBuildToStagingDirectory.Automation.cs and CustomStageCopyHandler.cs
	*/
	UPROPERTY(config)
	FString CustomStageCopyHandler;

private:
	
	/** The platform to LoadConfig for to get platform-specific packaging settings */
	FString ConfigPlatform;

public:

	// UObject Interface

	UE_API virtual void PostInitProperties() override;
#if WITH_EDITOR
	UE_API virtual void PostEditChangeProperty( struct FPropertyChangedEvent& PropertyChangedEvent ) override;
	UE_API virtual bool CanEditChange( const FProperty* InProperty ) const override;
#endif

	/** Gets a list of all valid packaging configurations for the current project */
	static UE_API TArray<EProjectPackagingBuildConfigurations> GetValidPackageConfigurations();

	/** Gets the current build target, checking that it's valid, and the default build target if it is not */
	UE_API const FTargetInfo* GetBuildTargetInfo() const;

	/** For non-default object instances, this will LoadConfig for a specific platform,*/
	void LoadSettingsForPlatform(FString PlatformName)
	{
		ConfigPlatform = PlatformName;
		LoadConfig();
	}

	bool GetUseZenStoreEffective() const
	{
		return bUseZenStore && bUseIoStore;
	}

	FString GetConfigPlatform() const
	{
		return ConfigPlatform;
	}

	virtual const TCHAR* GetConfigOverridePlatform() const override
	{
		return ConfigPlatform.Len() ? *ConfigPlatform : Super::GetConfigOverridePlatform();
	}

private:
	/** Fix up cooking paths after they've been edited or laoded */
	UE_API void FixCookingPaths();
};

#undef UE_API
