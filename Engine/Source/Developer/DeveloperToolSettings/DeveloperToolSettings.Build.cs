// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

public class DeveloperToolSettings : ModuleRules
{
	public DeveloperToolSettings(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.Add("Core");
		PrivateDependencyModuleNames.Add("Projects");
		PrivateDependencyModuleNames.Add("CoreUObject");
		PrivateDependencyModuleNames.Add("DeveloperSettings");
		PrivateDependencyModuleNames.Add("DesktopPlatform");
	}
}
