// Copyright Epic Games, Inc. All Rights Reserved.

#include "CookerSettings.h"
#include "UObject/UnrealType.h"
#include "HAL/IConsoleManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CookerSettings)

DEFINE_LOG_CATEGORY_STATIC(LogCookerSettings, Log, All);

UCookerSettings::UCookerSettings(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, bEnableCookOnTheSide(false)
	, bEnableBuildDDCInBackground(false)
	, bIterativeCookingForLaunchOn(false)
	, bCookOnTheFlyForLaunchOn(false)
	, CookProgressDisplayMode(ECookProgressDisplayMode::RemainingPackages)
	, bIgnoreIniSettingsOutOfDateForIteration(false)
	, bIgnoreScriptPackagesOutOfDateForIteration(false)
	, bCompileBlueprintsInDevelopmentMode(true)
	, BlueprintComponentDataCookingMethod(EBlueprintComponentDataCookingMethod::EnabledBlueprintsOnly)
	, bAllowCookedDataInEditorBuilds(false)
	, bCookBlueprintComponentTemplateData(false)
{
	SectionName = TEXT("Cooker");
	
	DefaultASTCQualityBySpeed = 2; // Medium preset
	DefaultASTCQualityBySize = 3;  // 6x6
	DefaultASTCQualityBySizeHQ = 4; // 4x4
}

void UCookerSettings::PostInitProperties()
{
	Super::PostInitProperties();
	UObject::UpdateClassesExcludedFromDedicatedServer(ClassesExcludedOnDedicatedServer, ModulesExcludedOnDedicatedServer);
	UObject::UpdateClassesExcludedFromDedicatedClient(ClassesExcludedOnDedicatedClient, ModulesExcludedOnDedicatedClient);

	// In the 'false' case, we previously deferred to the 'EnabledBlueprintsOnly' method, which is the current default, so we don't need to handle it here.
	if (bCookBlueprintComponentTemplateData)
	{
		BlueprintComponentDataCookingMethod = EBlueprintComponentDataCookingMethod::AllBlueprints;
	}
}

#if WITH_EDITOR

void UCookerSettings::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	static FName NAME_ClassesExcludedOnDedicatedServer = GET_MEMBER_NAME_CHECKED(UCookerSettings, ClassesExcludedOnDedicatedServer);
	static FName NAME_ClassesExcludedOnDedicatedClient = GET_MEMBER_NAME_CHECKED(UCookerSettings, ClassesExcludedOnDedicatedClient);

	static FName NAME_ModulesExcludedOnDedicatedServer(TEXT("ModulesExcludedOnDedicatedServer"));
	static FName NAME_ModulesExcludedOnDedicatedClient(TEXT("ModulesExcludedOnDedicatedClient"));

	static FName NAME_bCookBlueprintComponentTemplateData = GET_MEMBER_NAME_CHECKED(UCookerSettings, bCookBlueprintComponentTemplateData);
	static FName NAME_BlueprintComponentDataCookingMethod = GET_MEMBER_NAME_CHECKED(UCookerSettings, BlueprintComponentDataCookingMethod);

	if(PropertyChangedEvent.Property)
	{
		if(PropertyChangedEvent.Property->GetFName() == NAME_ClassesExcludedOnDedicatedServer
			|| PropertyChangedEvent.Property->GetFName() == NAME_ModulesExcludedOnDedicatedServer)
		{
			UObject::UpdateClassesExcludedFromDedicatedServer(ClassesExcludedOnDedicatedServer, ModulesExcludedOnDedicatedServer);
		}
		else if(PropertyChangedEvent.Property->GetFName() == NAME_ClassesExcludedOnDedicatedClient
			|| PropertyChangedEvent.Property->GetFName() == NAME_ModulesExcludedOnDedicatedClient)
		{
			UObject::UpdateClassesExcludedFromDedicatedClient(ClassesExcludedOnDedicatedClient, ModulesExcludedOnDedicatedClient);
		}
		else if (bCookBlueprintComponentTemplateData
			&& PropertyChangedEvent.Property->GetFName() == NAME_BlueprintComponentDataCookingMethod)
		{
			UE_LOGF(LogCookerSettings, Warning, "\'%ls\' has been deprecated in favor of \'%ls\', please remove \'%ls\' from %ls",
				*NAME_bCookBlueprintComponentTemplateData.ToString(),
				*NAME_BlueprintComponentDataCookingMethod.ToString(),
				*NAME_bCookBlueprintComponentTemplateData.ToString(),
				*GetDefaultConfigFilename());

			if (const FBoolProperty* OldProperty = FindFieldChecked<FBoolProperty>(GetClass(), NAME_bCookBlueprintComponentTemplateData))
			{
				OldProperty->SetPropertyValue_InContainer(this, false);
				UpdateSinglePropertyInConfigFile(OldProperty, GetDefaultConfigFilename());
			}
		}
		else
		{
			ExportValuesToConsoleVariables(PropertyChangedEvent.Property);
		}
	}
}

#endif
