// Copyright Epic Games, Inc. All Rights Reserved.

#include "DeveloperToolSettingsDelegates.h"
#include "Modules/ModuleManager.h"


IMPLEMENT_MODULE(FDefaultModuleImpl, DeveloperToolSettings);
