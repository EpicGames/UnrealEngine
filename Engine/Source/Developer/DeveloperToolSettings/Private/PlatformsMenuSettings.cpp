// Copyright Epic Games, Inc. All Rights Reserved.

#include "Settings/PlatformsMenuSettings.h"
#include "Misc/App.h"
#include "Misc/ConfigCacheIni.h"
#include "Interfaces/IProjectManager.h"
#include "DesktopPlatformModule.h"
#include "DeveloperToolSettingsDelegates.h"
#include "InstalledPlatformInfo.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PlatformsMenuSettings)

#define LOCTEXT_NAMESPACE "SettingsClasses"

extern const FTargetInfo* FindBestTargetInfo(const FString& TargetName, bool bContentOnlyUsesEngineTargets, bool* bOutIsProjectTarget);


UPlatformsMenuSettings::UPlatformsMenuSettings(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}


const FTargetInfo* UPlatformsMenuSettings::GetBuildTargetInfo(const FString& TargetName, bool& bOutIsProjectTarget) const
{
	return FindBestTargetInfo(TargetName, true, &bOutIsProjectTarget);
}

FName UPlatformsMenuSettings::GetTargetFlavorForPlatform(FName IniPlatformName) const
{
	const FName* Value = PerPlatformTargetFlavorName.Find(IniPlatformName);

	// the flavor name is also the name of the vanilla info
	return Value == nullptr ? IniPlatformName : *Value;
}

void UPlatformsMenuSettings::SetTargetFlavorForPlatform(FName IniPlatformName, FName TargetFlavorName)
{
	PerPlatformTargetFlavorName.Add(IniPlatformName, TargetFlavorName);
}

FString UPlatformsMenuSettings::GetArchitectureForPlatform(FName IniPlatformName) const
{
	const FString* Value = PerPlatformArchitecture.Find(IniPlatformName);
	return Value == nullptr ? TEXT("") : *Value;
}

void UPlatformsMenuSettings::SetArchitectureForPlatform(FName IniPlatformName, FString ArchitectureName)
{
	PerPlatformArchitecture.Add(IniPlatformName, ArchitectureName);
}

const FTargetInfo* UPlatformsMenuSettings::GetLaunchOnTargetInfo() const
{
	return FindBestTargetInfo(LaunchOnTarget, true, nullptr);
}

EProjectPackagingBuildConfigurations UPlatformsMenuSettings::GetLaunchOnBuildConfiguration() const
{
	// PPBC_MAX is used to indicate use the project default.
	return LaunchOnBuildConfiguration.IsSet() ? LaunchOnBuildConfiguration.GetValue() : EProjectPackagingBuildConfigurations::PPBC_MAX;
}

void UPlatformsMenuSettings::SetLaunchOnBuildConfiguration(EProjectPackagingBuildConfigurations BuildConfiguration)
{
	// PPBC_MAX is a special value to indicate use the project default.
	if (BuildConfiguration == EProjectPackagingBuildConfigurations::PPBC_MAX)
	{
		LaunchOnBuildConfiguration.Reset();
	}
	else
	{
		LaunchOnBuildConfiguration = BuildConfiguration;
	}
}

EProjectPackagingBuildConfigurations UPlatformsMenuSettings::GetPackageBuildConfiguration() const
{
	// PPBC_MAX is used to indicate use the project default.
	return PackageBuildConfiguration.IsSet() ? PackageBuildConfiguration.GetValue() : EProjectPackagingBuildConfigurations::PPBC_MAX;
}

void UPlatformsMenuSettings::SetPackageBuildConfiguration(EProjectPackagingBuildConfigurations BuildConfiguration)
{
	// PPBC_MAX is a special value to indicate use the project default.
	if (BuildConfiguration == EProjectPackagingBuildConfigurations::PPBC_MAX)
	{
		PackageBuildConfiguration.Reset();
	}
	else
	{
		PackageBuildConfiguration = BuildConfiguration;
	}
}

FName UPlatformsMenuSettings::GetPlatformForCustomBuild(ECustomBuildType Type, const FString& CustomBuildName) const
{
	const FName* IniPlatformName = nullptr;

	if (Type == ECustomBuildType::EngineCustomBuilds)
	{
		IniPlatformName = PerEngineCustomBuildPlatformName.Find(CustomBuildName);
	}
	else if (Type == ECustomBuildType::ProjectCustomBuilds)
	{
		IniPlatformName = PerProjectCustomBuildPlatformName.Find(CustomBuildName);
	}
	else
	{
		checkNoEntry();
	}

	return IniPlatformName == nullptr ? NAME_None : *IniPlatformName;
}

void UPlatformsMenuSettings::SetPlatformForCustomBuild(ECustomBuildType Type, const FString& CustomBuildName, FName IniPlatformName)
{
	if (Type == ECustomBuildType::EngineCustomBuilds)
	{
		PerEngineCustomBuildPlatformName.Add(CustomBuildName, IniPlatformName);
	}
	else if (Type == ECustomBuildType::ProjectCustomBuilds)
	{
		PerProjectCustomBuildPlatformName.Add(CustomBuildName, IniPlatformName);
	}
	else
	{
		checkNoEntry();
	}
}

FString UPlatformsMenuSettings::GetDeviceIdForCustomBuild(ECustomBuildType Type, const FString& CustomBuildName) const
{
	const FString* DeviceId = nullptr;

	if (Type == ECustomBuildType::EngineCustomBuilds)
	{
		DeviceId = PerEngineCustomBuildDeviceId.Find(CustomBuildName);
	}
	else if (Type == ECustomBuildType::ProjectCustomBuilds)
	{
		DeviceId = PerProjectCustomBuildDeviceId.Find(CustomBuildName);
	}
	else
	{
		checkNoEntry();
	}

	return DeviceId == nullptr ? FString() : *DeviceId;
}

void UPlatformsMenuSettings::SetDeviceIdForCustomBuild(ECustomBuildType Type, const FString& CustomBuildName, const FString& DeviceId)
{
	if (Type == ECustomBuildType::EngineCustomBuilds)
	{
		PerEngineCustomBuildDeviceId.Add(CustomBuildName, DeviceId);
	}
	else if (Type == ECustomBuildType::ProjectCustomBuilds)
	{
		PerProjectCustomBuildDeviceId.Add(CustomBuildName, DeviceId);
	}
	else
	{
		checkNoEntry();
	}
}

EProjectPackagingBuildConfigurations UPlatformsMenuSettings::GetBuildConfigurationForCustomBuild(ECustomBuildType Type, const FString& CustomBuildName) const
{
	const EProjectPackagingBuildConfigurations* BuildConfiguration = nullptr;

	if (Type == ECustomBuildType::EngineCustomBuilds)
	{
		BuildConfiguration = PerEngineCustomBuildBuildConfiguration.Find(CustomBuildName);
	}
	else if (Type == ECustomBuildType::ProjectCustomBuilds)
	{
		BuildConfiguration = PerProjectCustomBuildBuildConfiguration.Find(CustomBuildName);
	}
	else
	{
		checkNoEntry();
	}

	return BuildConfiguration == nullptr ? EProjectPackagingBuildConfigurations::PPBC_MAX : *BuildConfiguration;
}

void UPlatformsMenuSettings::SetBuildConfigurationForCustomBuild(ECustomBuildType Type, const FString& CustomBuildName, EProjectPackagingBuildConfigurations BuildConfiguration)
{
	if (Type == ECustomBuildType::EngineCustomBuilds)
	{
		PerEngineCustomBuildBuildConfiguration.Add(CustomBuildName, BuildConfiguration);
	}
	else if (Type == ECustomBuildType::ProjectCustomBuilds)
	{
		PerProjectCustomBuildBuildConfiguration.Add(CustomBuildName, BuildConfiguration);
	}
	else
	{
		checkNoEntry();
	}
}

FString UPlatformsMenuSettings::GetBuildTargetForCustomBuild(ECustomBuildType Type, const FString& CustomBuildName) const
{
	const FString* TargetName = nullptr;

	if (Type == ECustomBuildType::EngineCustomBuilds)
	{
		TargetName = PerEngineCustomBuildBuildTargetName.Find(CustomBuildName);
	}
	else if (Type == ECustomBuildType::ProjectCustomBuilds)
	{
		TargetName = PerProjectCustomBuildBuildTargetName.Find(CustomBuildName);
	}
	else
	{
		checkNoEntry();
	}

	return TargetName == nullptr ? FString() : *TargetName;
}

void UPlatformsMenuSettings::SetBuildTargetForCustomBuild(ECustomBuildType Type, const FString& CustomBuildName, const FString& TargetName)
{
	if (Type == ECustomBuildType::EngineCustomBuilds)
	{
		PerEngineCustomBuildBuildTargetName.Add(CustomBuildName, TargetName);
	}
	else if (Type == ECustomBuildType::ProjectCustomBuilds)
	{
		PerProjectCustomBuildBuildTargetName.Add(CustomBuildName, TargetName);
	}
	else
	{
		checkNoEntry();
	}
}

#undef LOCTEXT_NAMESPACE

