// Copyright Epic Games, Inc. All Rights Reserved.

#include "Settings/ProjectPackagingSettings.h"
#include "Misc/App.h"
#include "Misc/ConfigCacheIni.h"
#include "Interfaces/IProjectManager.h"
#include "DesktopPlatformModule.h"
#include "DeveloperToolSettingsDelegates.h"
#include "InstalledPlatformInfo.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ProjectPackagingSettings)

#define LOCTEXT_NAMESPACE "SettingsClasses"

DEFINE_LOG_CATEGORY_STATIC(LogProjectPackagingSettings, Log, Log);

/* UProjectPackagingSettings interface
 *****************************************************************************/

const UProjectPackagingSettings::FConfigurationInfo UProjectPackagingSettings::ConfigurationInfo[(int)EProjectPackagingBuildConfigurations::PPBC_MAX] =
{
	/* PPBC_Debug */         { EBuildConfiguration::Debug, LOCTEXT("DebugConfiguration", "Debug"), LOCTEXT("DebugConfigurationTooltip", "Package the game in Debug configuration") },
	/* PPBC_DebugGame */     { EBuildConfiguration::DebugGame, LOCTEXT("DebugGameConfiguration", "DebugGame"), LOCTEXT("DebugGameConfigurationTooltip", "Package the game in DebugGame configuration") },
	/* PPBC_Development */   { EBuildConfiguration::Development, LOCTEXT("DevelopmentConfiguration", "Development"), LOCTEXT("DevelopmentConfigurationTooltip", "Package the game in Development configuration") },
	/* PPBC_Test */          { EBuildConfiguration::Test, LOCTEXT("TestConfiguration", "Test"), LOCTEXT("TestConfigurationTooltip", "Package the game in Test configuration") },
	/* PPBC_Shipping */      { EBuildConfiguration::Shipping, LOCTEXT("ShippingConfiguration", "Shipping"), LOCTEXT("ShippingConfigurationTooltip", "Package the game in Shipping configuration") },
};

UProjectPackagingSettings::UProjectPackagingSettings(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}


void UProjectPackagingSettings::PostInitProperties()
{
	FixCookingPaths();

	Super::PostInitProperties();

	if (bUseZenStore && !bUseIoStore)
	{
		UE_LOGF(LogProjectPackagingSettings, Warning, "bUseZenStore is enabled in your project packaging settings while bUseIoStore is disabled. "
															"This is not a supported combination.  bUseZenStore is being treated as disabled. "
															"To prevent this warning, either (preferred) set bUseIoStore=True or set bUseZenStore=False"
															"in your project packaging settings in your DefaultGame.ini");
	}
}

void UProjectPackagingSettings::FixCookingPaths()
{
	// Fix AlwaysCook/NeverCook paths to use content root
	for (FDirectoryPath& PathToFix : DirectoriesToAlwaysCook)
	{
		if (!PathToFix.Path.IsEmpty() && !PathToFix.Path.StartsWith(TEXT("/"), ESearchCase::CaseSensitive))
		{
			PathToFix.Path = FString::Printf(TEXT("/Game/%s"), *PathToFix.Path);
		}
	}

	for (FDirectoryPath& PathToFix : DirectoriesToNeverCook)
	{
		if (!PathToFix.Path.IsEmpty() && !PathToFix.Path.StartsWith(TEXT("/"), ESearchCase::CaseSensitive))
		{
			PathToFix.Path = FString::Printf(TEXT("/Game/%s"), *PathToFix.Path);
		}
	}

	for (FDirectoryPath& PathToFix : TestDirectoriesToNotSearch)
	{
		if (!PathToFix.Path.IsEmpty() && !PathToFix.Path.StartsWith(TEXT("/"), ESearchCase::CaseSensitive))
		{
			PathToFix.Path = FString::Printf(TEXT("/Game/%s"), *PathToFix.Path);
		}
	}
}

#if WITH_EDITOR
void UProjectPackagingSettings::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	const FName Name = (PropertyChangedEvent.MemberProperty != nullptr)
		? PropertyChangedEvent.MemberProperty->GetFName()
		: NAME_None;

	// there is an issue where the default BuildConfig doesn't get propagated to the per-platform packaging settings instances
	// because their ConfigSystem doesn't get updated - so we do the nuclear option here and wipe them all - they will get re-loaded
	// when queried next. we just do it for all properties because any one of these could need the per-platform version to update
	// this code runs before the ReloadConfig is called on each instance
	if (Name != NAME_None)
	{
		FConfigCacheIni::ClearOtherPlatformConfigs();
	}

	if (Name == FName(TEXT("DirectoriesToAlwaysCook")) || Name == FName(TEXT("DirectoriesToNeverCook")) || Name == FName(TEXT("TestDirectoriesToNotSearch")) || Name == NAME_None)
	{
		// We need to fix paths for no name updates to catch the reloadconfig call
		FixCookingPaths();
	}
	else if (Name == FName(TEXT("ForDistribution")))
	{
		if (ForDistribution && BuildConfiguration != EProjectPackagingBuildConfigurations::PPBC_Shipping)
		{
			BuildConfiguration = EProjectPackagingBuildConfigurations::PPBC_Shipping;
			// force serialization for "Build COnfiguration"
			UpdateSinglePropertyInConfigFile(GetClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UProjectPackagingSettings, BuildConfiguration)), GetDefaultConfigFilename());
		}
	}
	else if (Name == FName(TEXT("bGenerateChunks")))
	{
		if (bGenerateChunks)
		{
			UsePakFile = true;
		}
	}
	else if (Name == FName(TEXT("UsePakFile")))
	{
		if (!UsePakFile)
		{
			bGenerateChunks = false;
			bBuildHttpChunkInstallData = false;
		}
	}
	else if (Name == FName(TEXT("bBuildHTTPChunkInstallData")))
	{
		if (bBuildHttpChunkInstallData)
		{
			UsePakFile = true;
			bGenerateChunks = true;
			//Ensure data is something valid
			if (HttpChunkInstallDataDirectory.Path.IsEmpty())
			{
				auto CloudInstallDir = FPaths::ConvertRelativePathToFull(FPaths::GetPath(FPaths::GetProjectFilePath())) / TEXT("ChunkInstall");
				HttpChunkInstallDataDirectory.Path = CloudInstallDir;
			}
			if (HttpChunkInstallDataVersion.IsEmpty())
			{
				HttpChunkInstallDataVersion = TEXT("release1");
			}
		}
	}
	else if (Name == FName((TEXT("ApplocalPrerequisitesDirectory"))))
	{
		// If a variable is already in use, assume the user knows what they are doing and don't modify the path
		if (!ApplocalPrerequisitesDirectory.Path.Contains("$("))
		{
			// Try making the path local to either project or engine directories.
			FString EngineRootedPath = ApplocalPrerequisitesDirectory.Path;
			FString EnginePath = FPaths::ConvertRelativePathToFull(FPaths::GetPath(FPaths::EngineDir())) + "/";
			FPaths::MakePathRelativeTo(EngineRootedPath, *EnginePath);
			if (FPaths::IsRelative(EngineRootedPath))
			{
				ApplocalPrerequisitesDirectory.Path = "$(EngineDir)/" + EngineRootedPath;
				return;
			}

			FString ProjectRootedPath = ApplocalPrerequisitesDirectory.Path;
			FString ProjectPath = FPaths::ConvertRelativePathToFull(FPaths::GetPath(FPaths::GetProjectFilePath())) + "/";
			FPaths::MakePathRelativeTo(ProjectRootedPath, *ProjectPath);
			if (FPaths::IsRelative(EngineRootedPath))
			{
				ApplocalPrerequisitesDirectory.Path = "$(ProjectDir)/" + ProjectRootedPath;
				return;
			}
		}
	}
}

bool UProjectPackagingSettings::CanEditChange(const FProperty* InProperty) const
{
	return Super::CanEditChange(InProperty);
}
#endif

TArray<EProjectPackagingBuildConfigurations> UProjectPackagingSettings::GetValidPackageConfigurations()
{
	// Check if the project has code
	FProjectStatus ProjectStatus;
	bool bHasCode = IProjectManager::Get().QueryStatusForCurrentProject(ProjectStatus) && ProjectStatus.bCodeBasedProject;
	EProjectType ProjectType = bHasCode ? EProjectType::Code : EProjectType::Content;

	// If if does, find all the targets
	const TArray<FTargetInfo>* Targets = nullptr;
	if (bHasCode)
	{
		Targets = &(FDesktopPlatformModule::Get()->GetTargetsForCurrentProject());
	}

	// Set up all the configurations
	TArray<EProjectPackagingBuildConfigurations> Configurations;
	for (int32 Idx = 0; Idx < (int)EProjectPackagingBuildConfigurations::PPBC_MAX; Idx++)
	{
		EProjectPackagingBuildConfigurations PackagingConfiguration = (EProjectPackagingBuildConfigurations)Idx;

		// Check the target type is valid
		const UProjectPackagingSettings::FConfigurationInfo& Info = UProjectPackagingSettings::ConfigurationInfo[Idx];
		if (FInstalledPlatformInfo::Get().IsValid(TOptional<EBuildTargetType>(), TOptional<FString>(), Info.Configuration, ProjectType, EInstalledPlatformState::Downloaded))
		{
			Configurations.Add(PackagingConfiguration);
		}
	}
	return Configurations;
}


const FTargetInfo* FindBestTargetInfo(const FString& TargetName, bool bContentOnlyUsesEngineTargets, bool* bOutIsProjectTarget=nullptr)
{
	bool bUseEngineTargets = false;
	if (bContentOnlyUsesEngineTargets)
	{
		// Collect build targets. Content-only projects use Engine targets.
		FProjectStatus ProjectStatus;
		bUseEngineTargets =!(IProjectManager::Get().QueryStatusForCurrentProject(ProjectStatus) && ProjectStatus.bCodeBasedProject);
	}

	if (bOutIsProjectTarget != nullptr)
	{
		*bOutIsProjectTarget = !bUseEngineTargets;
	}

	IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
	const TArray<FTargetInfo>& TargetsRef = bUseEngineTargets ? DesktopPlatform->GetTargetsForProject(FString()) : DesktopPlatform->GetTargetsForCurrentProject();

	const FTargetInfo* DefaultGameTarget = nullptr;
	const FTargetInfo* DefaultClientTarget = nullptr;
	for (const FTargetInfo& Target : TargetsRef)
	{
		if (Target.Name == TargetName)
		{
			return &Target;
		}
		else if (Target.Type == EBuildTargetType::Game && (DefaultGameTarget == nullptr || Target.Name < DefaultGameTarget->Name))
		{
			DefaultGameTarget = &Target;
		}
		else if (Target.Type == EBuildTargetType::Client && (DefaultClientTarget == nullptr || Target.Name < DefaultClientTarget->Name))
		{
			DefaultClientTarget = &Target;
		}
	}

	// Added in 5.7.2 to help diagnose UE-316569
	if (DefaultGameTarget == nullptr && DefaultClientTarget == nullptr)
	{
		UE_LOGF(LogProjectPackagingSettings, Log, "UProjectPackagingSettings FindBestTargetInfo for '%ls' resulted in null FTargetInfo*.  Listing targets that were searched:", *TargetName);
		for (const FTargetInfo& Target : TargetsRef)
		{
			UE_LOGF(LogProjectPackagingSettings, Log, "  %i %ls", EnumToUnderlyingType(Target.Type), *Target.Name);
		}
		UE_LOGF(LogProjectPackagingSettings, Log, "  End of target list.");
	}

	return (DefaultGameTarget != nullptr) ? DefaultGameTarget : DefaultClientTarget;

}


const FTargetInfo* UProjectPackagingSettings::GetBuildTargetInfo() const
{
	return FindBestTargetInfo(BuildTarget, false);
}

#undef LOCTEXT_NAMESPACE

