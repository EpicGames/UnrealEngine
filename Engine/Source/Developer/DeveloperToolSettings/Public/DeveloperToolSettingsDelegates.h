// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

/**
  * FDeveloperToolSettingsDelegates
  * Delegates that are needed for developer tools to talk to engine/editor
  **/
struct FDeveloperToolSettingsDelegates
{
	
};

